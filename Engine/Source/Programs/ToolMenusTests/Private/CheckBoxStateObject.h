// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "ToolMenuContext.h"

#include "CheckBoxStateObject.generated.h"

UCLASS()
class UCheckBoxStateObject : public UObject
{
	GENERATED_BODY()

public:
	UCheckBoxStateObject() = default;

	void SetStateToReturn(const ECheckBoxState InState)
	{
		StateToReturn = InState;
	}

	UFUNCTION()
	ECheckBoxState GetActionCheckState(const FToolMenuContext& InContext) const
	{
		return StateToReturn;
	}

private:
	ECheckBoxState StateToReturn;
};
