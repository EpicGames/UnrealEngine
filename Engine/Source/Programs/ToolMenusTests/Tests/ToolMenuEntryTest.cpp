// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenuEntry.h"

#include "CheckBoxStateObject.h"
#include "TestHarness.h"
#include "ToolMenus.h"

TEST_CASE("Developer::ToolMenus::UToolMenuEntry::GetCheckState supports commands", "[ToolMenus]")
{
	GIVEN("A command list with a command bound to return Checked")
	{
		const TSharedPtr<FUICommandInfo> CommandInfo = MakeShared<FUICommandInfo>("MyTestContext");
		const TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();
		CommandList->MapAction(
			CommandInfo,
			FExecuteAction(),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda(
				[]() -> ECheckBoxState
				{
					return ECheckBoxState::Checked;
				}
			)
		);

		AND_GIVEN("A menu entry initialized with the command and an FToolMenuContext that includes the command list")
		{
			FToolMenuContext Context;
			Context.AppendCommandList(CommandList);

			const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(CommandInfo);

			THEN("FToolMenuEntry::GetCheckState returns Checked")
			{
				CHECK(Entry.GetCheckState(Context) == ECheckBoxState::Checked);
			}
		}

		AND_GIVEN("A menu entry initialized with both the command list and the command")
		{
			FToolMenuContext Context;

			const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntryWithCommandList(CommandInfo, CommandList);

			THEN("FToolMenuEntry::GetCheckState returns Checked")
			{
				CHECK(Entry.GetCheckState(Context) == ECheckBoxState::Checked);
			}
		}
	}

	GIVEN("A command list with a command bound to return Unchecked")
	{
		const TSharedPtr<FUICommandInfo> CommandInfo = MakeShared<FUICommandInfo>("MyTestContext");
		const TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();
		CommandList->MapAction(
			CommandInfo,
			FExecuteAction(),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda(
				[]() -> ECheckBoxState
				{
					return ECheckBoxState::Unchecked;
				}
			)
		);

		AND_GIVEN("A menu entry initialized with the command and an FToolMenuContext that includes the command list")
		{
			FToolMenuContext Context;
			Context.AppendCommandList(CommandList);

			const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(CommandInfo);

			THEN("FToolMenuEntry::GetCheckState returns Unchecked")
			{
				CHECK(Entry.GetCheckState(Context) == ECheckBoxState::Unchecked);
			}
		}

		AND_GIVEN("A menu entry initialized with both the command list and the command")
		{
			FToolMenuContext Context;

			const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntryWithCommandList(CommandInfo, CommandList);

			THEN("FToolMenuEntry::GetCheckState returns Unchecked")
			{
				CHECK(Entry.GetCheckState(Context) == ECheckBoxState::Unchecked);
			}
		}
	}
}

TEST_CASE("Developer::ToolMenus::UToolMenuEntry::GetCheckState supports FUIAction", "[ToolMenus]")
{
	GIVEN("An entry with a FUIAction with a GetActionCheckState that returns true")
	{
		const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			NAME_None,
			FText(),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[]() -> bool
					{
						return true;
					}
				)
			)
		);

		THEN("FToolMenuEntry::GetCheckState returns Checked")
		{
			CHECK(Entry.GetCheckState(FToolMenuContext()) == ECheckBoxState::Checked);
		}
	}

	GIVEN("An entry with a FUIAction with a GetActionCheckState that returns false")
	{
		const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			NAME_None,
			FText(),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda(
					[]() -> bool
					{
						return false;
					}
				)
			)
		);

		THEN("FToolMenuEntry::GetCheckState returns Checked")
		{
			CHECK(Entry.GetCheckState(FToolMenuContext()) == ECheckBoxState::Unchecked);
		}
	}
}

TEST_CASE("Developer::ToolMenus::UToolMenuEntry::GetCheckState supports FToolUIAction", "[ToolMenus]")
{
	GIVEN("An entry with an FToolUIAction with a GetActionCheckState that returns Checked")
	{
		FToolUIAction Action;
		Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda(
			[](const FToolMenuContext&) -> ECheckBoxState
			{
				return ECheckBoxState::Checked;
			}
		);

		const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(NAME_None, FText(), FText(), FSlateIcon(), Action);

		THEN("FToolMenuEntry::GetCheckState returns Checked")
		{
			CHECK(Entry.GetCheckState(FToolMenuContext()) == ECheckBoxState::Checked);
		}
	}

	GIVEN("An entry with an FToolUIAction with a GetActionCheckState that returns Checked")
	{
		FToolUIAction Action;
		Action.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda(
			[](const FToolMenuContext&) -> ECheckBoxState
			{
				return ECheckBoxState::Unchecked;
			}
		);

		const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(NAME_None, FText(), FText(), FSlateIcon(), Action);

		THEN("FToolMenuEntry::GetCheckState returns Unchecked")
		{
			CHECK(Entry.GetCheckState(FToolMenuContext()) == ECheckBoxState::Unchecked);
		}
	}
}

TEST_CASE("Developer::ToolMenus::UToolMenuEntry::GetCheckState supports FToolDynamicUIAction", "[ToolMenus]")
{
	GIVEN("An entry with an FToolDynamicUIAction with a GetActionCheckState that returns Checked")
	{
		UCheckBoxStateObject* CheckBoxStateObject = NewObject<UCheckBoxStateObject>();
		CheckBoxStateObject->SetStateToReturn(ECheckBoxState::Checked);

		FToolDynamicUIAction Action;
		Action.GetActionCheckState.BindUFunction(CheckBoxStateObject, "GetActionCheckState");

		const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(NAME_None, FText(), FText(), FSlateIcon(), Action);

		THEN("FToolMenuEntry::GetCheckState returns Checked")
		{
			CHECK(Entry.GetCheckState(FToolMenuContext()) == ECheckBoxState::Checked);
		}
	}
	GIVEN("An entry with an FToolDynamicUIAction with a GetActionCheckState that returns Unchecked")
	{
		UCheckBoxStateObject* CheckBoxStateObject = NewObject<UCheckBoxStateObject>();
		CheckBoxStateObject->SetStateToReturn(ECheckBoxState::Unchecked);

		FToolDynamicUIAction Action;
		Action.GetActionCheckState.BindUFunction(CheckBoxStateObject, "GetActionCheckState");

		const FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(NAME_None, FText(), FText(), FSlateIcon(), Action);

		THEN("FToolMenuEntry::GetCheckState returns Unchecked")
		{
			CHECK(Entry.GetCheckState(FToolMenuContext()) == ECheckBoxState::Unchecked);
		}
	}
}
