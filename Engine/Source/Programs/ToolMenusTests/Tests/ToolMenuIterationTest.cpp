// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenuIteration.h"

#include "TestHarness.h"
#include "ToolMenus.h"

TEST_CASE("Developer::ToolMenus::FToolMenuIteration does not visit anything for non-existant menus", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	GIVEN("The menu we try to visist does not exist")
	{
		const FName MenuName = "MenuNameThatDoesNotExist";

		int32 NumVisists = 0;
		UE::ToolMenus::VisitMenuEntries(
			UToolMenus::Get(),
			MenuName,
			FToolMenuContext(),
			UE::ToolMenus::FToolMenuVisitor::CreateLambda(
				[&NumVisists](const UE::ToolMenus::FToolMenuIterationInfo& Info) -> bool
				{
					++NumVisists;

					return true;
				}
			)
		);

		THEN("We did not visit any entries")
		{
			CHECK(NumVisists == 0);
		}
	}
}

TEST_CASE("Developer::ToolMenus::FToolMenuIteration can visit menus", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName = "ToolMenuIterationTest_MyMenu";
	UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(MenuName);

	GIVEN("An empty menu")
	{
		WHEN("The menu is iterated")
		{
			int32 NumVisists = 0;
			UE::ToolMenus::VisitMenuEntries(
				UToolMenus::Get(),
				MenuName,
				FToolMenuContext(),
				UE::ToolMenus::FToolMenuVisitor::CreateLambda(
					[&NumVisists](const UE::ToolMenus::FToolMenuIterationInfo& Info) -> bool
					{
						++NumVisists;

						return true;
					}
				)
			);

			THEN("No entries are visisted")
			{
				CHECK(NumVisists == 0);
			}
		}
	}

	GIVEN("A menu with one entry")
	{
		FToolMenuSection& Section = Menu->FindOrAddSection(NAME_None);
		Section.AddMenuEntry("MyEntry", FText(), FText(), FSlateIcon(), FUIAction());

		WHEN("The menu is iterated")
		{
			int32 NumVisists = 0;
			UE::ToolMenus::VisitMenuEntries(
				UToolMenus::Get(),
				MenuName,
				FToolMenuContext(),
				UE::ToolMenus::FToolMenuVisitor::CreateLambda(
					[&NumVisists](const UE::ToolMenus::FToolMenuIterationInfo& Info) -> bool
					{
						++NumVisists;

						return true;
					}
				)
			);

			THEN("One entry is visited")
			{
				CHECK(NumVisists == 1);
			}
		}
	}
	GIVEN("A menu with six entries across two sections")
	{
		{
			FToolMenuSection& Section = Menu->FindOrAddSection(NAME_None);
			Section.AddMenuEntry("A", FText(), FText(), FSlateIcon(), FUIAction());
			Section.AddMenuEntry("B", FText(), FText(), FSlateIcon(), FUIAction());
			Section.AddMenuEntry("C", FText(), FText(), FSlateIcon(), FUIAction());
		}

		{
			FToolMenuSection& Section = Menu->FindOrAddSection("Foo");
			Section.AddMenuEntry("D", FText(), FText(), FSlateIcon(), FUIAction());
			Section.AddMenuEntry("E", FText(), FText(), FSlateIcon(), FUIAction());
			Section.AddMenuEntry("F", FText(), FText(), FSlateIcon(), FUIAction());
		}

		WHEN("The menu is iterated")
		{
			TSet<FName> VisistedEntryNames;
			TSet<FName> VisistedSectionNames;

			int32 NumVisists = 0;
			UE::ToolMenus::VisitMenuEntries(
				UToolMenus::Get(),
				MenuName,
				FToolMenuContext(),
				UE::ToolMenus::FToolMenuVisitor::CreateLambda(
					[&NumVisists, &VisistedEntryNames, &VisistedSectionNames](const UE::ToolMenus::FToolMenuIterationInfo& Info
					) -> bool
					{
						++NumVisists;
						VisistedEntryNames.Add(Info.Entry.Name);
						VisistedSectionNames.Add(Info.Section.Name);

						return true;
					}
				)
			);

			THEN("Six entries are found")
			{
				CHECK(NumVisists == 6);
			}

			THEN("The extected entry names were visisted")
			{
				CHECK(VisistedEntryNames.Contains("A"));
				CHECK(VisistedEntryNames.Contains("B"));
				CHECK(VisistedEntryNames.Contains("C"));

				CHECK(VisistedEntryNames.Contains("D"));
				CHECK(VisistedEntryNames.Contains("E"));
				CHECK(VisistedEntryNames.Contains("F"));
			}

			THEN("The extected section names were visisted")
			{
				CHECK(VisistedSectionNames.Contains(NAME_None));
				CHECK(VisistedSectionNames.Contains("Foo"));
			}
		}
	}
}

TEST_CASE("Developer::ToolMenus::FToolMenuIteration can visit menu extensions", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName = "ToolMenuIterationTest_MyMenu";
	UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(MenuName);

	GIVEN("A menu with one entry")
	{
		FToolMenuSection& Section = Menu->FindOrAddSection(NAME_None);
		Section.AddMenuEntry("A", FText(), FText(), FSlateIcon(), FUIAction());

		AND_GIVEN("The menu is extended with another entry")
		{
			{
				UToolMenu* const ExtendedMenu = UToolMenus::Get()->ExtendMenu(MenuName);
				FToolMenuSection& SomeSection = ExtendedMenu->FindOrAddSection("SomeSection");
				SomeSection.AddMenuEntry("B", FText(), FText(), FSlateIcon(), FUIAction());
			}

			WHEN("The menu is iterated")
			{
				TSet<FName> VisistedEntryNames;

				int32 NumVisists = 0;
				UE::ToolMenus::VisitMenuEntries(
					UToolMenus::Get(),
					MenuName,
					FToolMenuContext(),
					UE::ToolMenus::FToolMenuVisitor::CreateLambda(
						[&NumVisists, &VisistedEntryNames](const UE::ToolMenus::FToolMenuIterationInfo& Info) -> bool
						{
							++NumVisists;
							VisistedEntryNames.Add(Info.Entry.Name);

							return true;
						}
					)
				);

				THEN("Two entries are found")
				{
					CHECK(NumVisists == 2);
				}

				THEN("The entry of the base menu is found")
				{
					CHECK(VisistedEntryNames.Contains("A"));
				}

				THEN("The entry of the extension is found")
				{
					CHECK(VisistedEntryNames.Contains("B"));
				}
			}
		}
	}
}

TEST_CASE("Developer::ToolMenus::FToolMenuIteration can visit submenu extensions", "[ToolMenus]")
{
	// We have to use the global UToolMenus instance because UToolMenu::GetMenuCustomizationHierarchy calls UToolMenus::Get().
	UToolMenus* const ToolMenus = UToolMenus::Get();

	const FName MenuName = "ToolMenuIterationTest_MyMenu";
	UToolMenu* const Menu = ToolMenus->RegisterMenu(MenuName);

	GIVEN("A menu with a submenu that has one entry")
	{
		const FName SubmenuName = "MySubmenu";
		{
			FToolMenuSection& Section = Menu->FindOrAddSection(NAME_None);

			FNewToolMenuDelegate MakeMenu = FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					FToolMenuSection& Section = InMenu->FindOrAddSection(NAME_None);
					Section.AddMenuEntry("one", FText(), FText(), FSlateIcon(), FUIAction());
				}
			);

			Section.AddSubMenu(SubmenuName, FText(), FText(), MakeMenu);
		}

		AND_GIVEN("The submenu is extended with another entry")
		{
			{
				const FName FullSubmenuName = UToolMenus::JoinMenuPaths(MenuName, SubmenuName);
				UToolMenu* const ExtendedSubmenu = ToolMenus->ExtendMenu(FullSubmenuName);
				FToolMenuSection& Section = ExtendedSubmenu->FindOrAddSection("Foo");
				Section.AddMenuEntry("two", FText(), FText(), FSlateIcon(), FUIAction());
			}

			WHEN("The menu is iterated")
			{
				TSet<FName> VisitedEntryNames;

				int32 NumVisists = 0;
				UE::ToolMenus::VisitMenuEntries(
					ToolMenus,
					MenuName,
					FToolMenuContext(),
					UE::ToolMenus::FToolMenuVisitor::CreateLambda(
						[&NumVisists, &VisitedEntryNames](const UE::ToolMenus::FToolMenuIterationInfo& Info) -> bool
						{
							++NumVisists;
							VisitedEntryNames.Add(Info.Entry.Name);

							return true;
						}
					)
				);

				THEN("Two entries are found")
				{
					CHECK(NumVisists == 2);
				}

				THEN("The entry of the base submenu is found")
				{
					CHECK(VisitedEntryNames.Contains("one"));
				}

				THEN("The entry of the extension to the submenu is found")
				{
					CHECK(VisitedEntryNames.Contains("two"));
				}
			}
		}
	}

	ToolMenus->RemoveMenu(MenuName);
}

TEST_CASE("Developer::ToolMenus::FToolMenuIteration can be stopped", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName = "ToolMenuIterationTest_MyMenu";
	UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(MenuName);

	GIVEN("A menu with two entries")
	{
		FToolMenuSection& Section = Menu->FindOrAddSection(NAME_None);
		Section.AddMenuEntry("one", FText(), FText(), FSlateIcon(), FUIAction());
		Section.AddMenuEntry("two", FText(), FText(), FSlateIcon(), FUIAction());

		WHEN("The menu is iterated and the visitor returns false to stop the iteration")
		{
			int32 NumVisits = 0;
			UE::ToolMenus::VisitMenuEntries(
				UToolMenus::Get(),
				MenuName,
				FToolMenuContext(),
				UE::ToolMenus::FToolMenuVisitor::CreateLambda(
					[&NumVisits](const UE::ToolMenus::FToolMenuIterationInfo& Info) -> bool
					{
						++NumVisits;

						return false;
					}
				)
			);

			THEN("One of the two entries is visisted")
			{
				CHECK(NumVisits == 1);
			}
		}
	}
}

TEST_CASE("Developer::ToolMenus::FToolMenuIteration can visit submenus", "[ToolMenus]")
{
	// We have to use the global UToolMenus instance because UToolMenu::GetMenuCustomizationHierarchy calls UToolMenus::Get().
	UToolMenus* const ToolMenus = UToolMenus::Get();

	const FName MenuName = "ToolMenuIterationTest_MyMenuWithSubmenu";
	UToolMenu* const Menu = ToolMenus->RegisterMenu(MenuName);

	GIVEN("A menu with a submenu that has one entry")
	{
		{
			FToolMenuSection& Section = Menu->FindOrAddSection(NAME_None);

			FNewToolMenuDelegate MakeMenu = FNewToolMenuDelegate::CreateLambda(
				[](UToolMenu* InMenu)
				{
					FToolMenuSection& Section = InMenu->FindOrAddSection(NAME_None);
					Section.AddMenuEntry("X", FText(), FText(), FSlateIcon(), FUIAction());
				}
			);

			Section.AddSubMenu("MySubmenu", FText(), FText(), MakeMenu);
		}

		WHEN("The menu is iterated")
		{
			TSet<FName> VisitedEntryNames;

			int32 NumVisits = 0;
			UE::ToolMenus::VisitMenuEntries(
				ToolMenus,
				MenuName,
				FToolMenuContext(),
				UE::ToolMenus::FToolMenuVisitor::CreateLambda(
					[&NumVisits, &VisitedEntryNames](const UE::ToolMenus::FToolMenuIterationInfo& Info) -> bool
					{
						++NumVisits;
						VisitedEntryNames.Add(Info.Entry.Name);

						return true;
					}
				)
			);

			THEN("One entry is visisted")
			{
				CHECK(NumVisits == 1);
			}

			THEN("The expected entry name was visisted")
			{
				CHECK(VisitedEntryNames.Contains("X"));
			}
		}
	}

	ToolMenus->RemoveMenu(MenuName);
}
