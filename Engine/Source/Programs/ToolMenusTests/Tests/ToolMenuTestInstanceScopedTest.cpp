// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHarness.h"

#include "ToolMenus.h"

TEST_CASE("UE::ToolMenus::FToolMenuTestInstanceScoped replaces the singleton while the scope is active", "[ToolMenus]")
{
	UToolMenus* const OriginalSingleton = UToolMenus::Get();

	{
		UE::ToolMenus::FToolMenuTestInstanceScoped Scope1;

		UToolMenus* const FirstNestedSingleton = UToolMenus::Get();

		CHECK(FirstNestedSingleton != OriginalSingleton);

		{
			UE::ToolMenus::FToolMenuTestInstanceScoped Scope2;

			UToolMenus* const SecondNestedSingleton = UToolMenus::Get();

			CHECK(SecondNestedSingleton != FirstNestedSingleton);
			CHECK(SecondNestedSingleton != OriginalSingleton);
		}
	}
}

TEST_CASE("UE::ToolMenus::FToolMenuTestInstanceScoped reinstates the original singleton when the scope is destructed", "[ToolMenus]")
{
	UToolMenus* const OriginalSingleton = UToolMenus::Get();

	{
		UE::ToolMenus::FToolMenuTestInstanceScoped Scope1;

		UToolMenus* const FirstNestedSingleton = UToolMenus::Get();

		{
			UE::ToolMenus::FToolMenuTestInstanceScoped Scope2;
		}

		CHECK(FirstNestedSingleton == UToolMenus::Get());
	}

	CHECK(OriginalSingleton == UToolMenus::Get());
}

const FName FToolMenuTestInstanceScopedMenuName =
	"FToolMenuTestInstanceScoped_prevents_names_from_leaking_between_tests";

TEST_CASE("UE::ToolMenus::FToolMenuTestInstanceScoped prevents menu names from leaking between tests (test 1 of 2)", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	CHECK(!UToolMenus::Get()->IsMenuRegistered(FToolMenuTestInstanceScopedMenuName));

	UToolMenus::Get()->RegisterMenu(FToolMenuTestInstanceScopedMenuName);
}

TEST_CASE("UE::ToolMenus::FToolMenuTestInstanceScoped prevents menu names from leaking between tests (test 2 of 2)", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	CHECK(!UToolMenus::Get()->IsMenuRegistered(FToolMenuTestInstanceScopedMenuName));

	UToolMenus::Get()->RegisterMenu(FToolMenuTestInstanceScopedMenuName);
}
