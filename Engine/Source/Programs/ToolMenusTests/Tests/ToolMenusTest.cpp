// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMenus.h"

#include "TestHarness.h"

TEST_CASE("Developer::ToolMenus::Can create menu", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu("ToolMenusTest_MyMenu");

	CHECK(ToolMenu);
}

TEST_CASE("Developer::ToolMenus::Non-registered menu is not registered", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	CHECK_FALSE(UToolMenus::Get()->IsMenuRegistered("ToolMenusTest_MyMenu"));
}

TEST_CASE("Developer::ToolMenus::Removed menu is not registered", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName("ToolMenusTest_MyMenu");

	UToolMenus::Get()->RegisterMenu(MenuName);

	CHECK(UToolMenus::Get()->IsMenuRegistered(MenuName));

	UToolMenus::Get()->RemoveMenu(MenuName);

	CHECK_FALSE(UToolMenus::Get()->IsMenuRegistered(MenuName));
}

TEST_CASE("Developer::ToolMenus::GenerateWidget calls dynamic section lambdas", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName("ToolMenusTest_MyMenu");

	UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);

	REQUIRE(ToolMenu);

	bool bWasLambdaCalled = false;
	ToolMenu->AddDynamicSection("MySection", FNewToolMenuDelegateLegacy::CreateLambda([&bWasLambdaCalled](FMenuBuilder&, UToolMenu*) {
		bWasLambdaCalled = true;
	}));

	UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext());

	CHECK(bWasLambdaCalled);
}

TEST_CASE("Developer::ToolMenus::Can add and remove sections to menu", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu("ToolMenusTest_MyMenu");

	REQUIRE(ToolMenu);

	// Add a section to the menu
	const FName SectionName("MySection");
	
	ToolMenu->AddSection(SectionName);
	CHECK(ToolMenu->Sections.Num() == 1);

	// Check if the section we added is correct
	FToolMenuSection* Section = ToolMenu->FindSection(SectionName);
	REQUIRE(Section);
	CHECK(Section->Name == SectionName);

	// Check if we are able to remove the section
	ToolMenu->RemoveSection(SectionName);
	CHECK(ToolMenu->Sections.Num() == 0);
}

TEST_CASE("Developer::ToolMenus::Can add entries to menu section", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu("ToolMenusTest_MyMenu");

	REQUIRE(ToolMenu);

	FToolMenuSection& Section = ToolMenu->AddSection("MySection");

	// Add the first entry to the menu
	const FName Entry1Name("MyEntry1");
	Section.AddMenuEntry(Entry1Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());
	CHECK(Section.Blocks.Num() == 1);

	// Check if the first entry is correct
	FToolMenuEntry* Entry1 = Section.FindEntry(Entry1Name);
	REQUIRE(Entry1);
	CHECK(Entry1->Name == Entry1Name);

	// Add the second entry to the menu
	const FName Entry2Name("MyEntry2");
	Section.AddMenuEntry(Entry2Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());
	CHECK(Section.Blocks.Num() == 2);

	// Check if the second entry is correct
	FToolMenuEntry* Entry2 = Section.FindEntry(Entry2Name);
	REQUIRE(Entry2);
	CHECK(Entry2->Name == Entry2Name);
}

TEST_CASE("Developer::ToolMenus::Can add runtime menu customization", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName("MyMenu");

	UToolMenus::Get()->RegisterMenu(MenuName);

	// Add the customization to the menu
	UToolMenus::Get()->AddRuntimeMenuCustomization(MenuName);

	// Try and find the customization
	FCustomizedToolMenu* CustomizedToolMenu = UToolMenus::Get()->FindRuntimeMenuCustomization(MenuName);
	CHECK(CustomizedToolMenu);
}


TEST_CASE("Developer::ToolMenus::Can customize menu using runtime menu customization", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName("MyMenu");

	UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);

	REQUIRE(ToolMenu);

	// Add test entries 1, 2 and 3
	const FName SectionName("MySection");
	const FName Entry1Name("MyEntry1");
	const FName Entry2Name("MyEntry2");
	const FName Entry3Name("MyEntry3");

	FToolMenuSection& Section = ToolMenu->AddSection(SectionName);
	Section.AddMenuEntry(Entry1Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());
	Section.AddMenuEntry(Entry2Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());
	Section.AddMenuEntry(Entry3Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());

	// Add the customization to the menu
	UToolMenus::Get()->AddRuntimeMenuCustomization(MenuName);
	FCustomizedToolMenu* CustomizedToolMenu = UToolMenus::Get()->FindRuntimeMenuCustomization(MenuName);
	REQUIRE(CustomizedToolMenu);

	// Test the customization by denying entry 2
	const FName OwnerName("ToolMenusTest");

	CustomizedToolMenu->MenuPermissions.AddDenyListItem(OwnerName, Entry2Name);

	UToolMenu* GeneratedMenu = UToolMenus::Get()->GenerateMenu(MenuName, FToolMenuContext());
	REQUIRE(GeneratedMenu);

	FToolMenuSection* GeneratedMenuSection = GeneratedMenu->FindSection(SectionName);
	REQUIRE(GeneratedMenuSection);

	// We should only have 2 entries: Entry1 and Entry3 since Entry2 is denied
	CHECK(GeneratedMenuSection->Blocks.Num() == 2);
	CHECK(GeneratedMenuSection->FindEntry(Entry1Name));
	CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry2Name));
	CHECK(GeneratedMenuSection->FindEntry(Entry3Name));
}

TEST_CASE("Developer::ToolMenus::Can add runtime menu profile", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName("MyMenu");

	UToolMenus::Get()->RegisterMenu(MenuName);

	// Add the customization to the menu
	const FName ProfileName("MyProfile1");
	UToolMenus::Get()->AddRuntimeMenuProfile(MenuName, ProfileName);

	// Try and find the profile
	FToolMenuProfile* MenuProfile = UToolMenus::Get()->FindRuntimeMenuProfile(MenuName, ProfileName);
	CHECK(MenuProfile);
}

TEST_CASE("Developer::ToolMenus::Can customize menu using runtime menu profile", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName("MyMenu");

	UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);

	REQUIRE(ToolMenu);

	// Add test entries 1, 2 and 3 to the menu
	const FName SectionName("MySection");
	const FName Entry1Name("MyEntry1");
	const FName Entry2Name("MyEntry2");
	const FName Entry3Name("MyEntry3");

	FToolMenuSection& Section = ToolMenu->AddSection(SectionName);
	Section.AddMenuEntry(Entry1Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());
	Section.AddMenuEntry(Entry2Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());
	Section.AddMenuEntry(Entry3Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());

	// Add a profile to the menu
	const FName Profile1Name("MyProfile1");

	UToolMenus::Get()->AddRuntimeMenuProfile(MenuName, Profile1Name);

	// Check if the profile was added
	FToolMenuProfile* MenuProfile = UToolMenus::Get()->FindRuntimeMenuProfile(MenuName, Profile1Name);
	REQUIRE(MenuProfile);

	// Deny Entry 2 in the profile
	const FName OwnerName("ToolMenusTest");
	
	MenuProfile->MenuPermissions.AddDenyListItem(OwnerName, Entry2Name);

	SECTION("Generating menu without profile should show all entries")
	{
		UToolMenu* GeneratedMenuWithoutProfile = UToolMenus::Get()->GenerateMenu(MenuName, FToolMenuContext());
		REQUIRE(GeneratedMenuWithoutProfile);

		FToolMenuSection* GeneratedMenuSection = GeneratedMenuWithoutProfile->FindSection(SectionName);
		REQUIRE(GeneratedMenuSection);

		CHECK(GeneratedMenuSection->Blocks.Num() == 3);
		CHECK(GeneratedMenuSection->FindEntry(Entry1Name));
		CHECK(GeneratedMenuSection->FindEntry(Entry2Name));
		CHECK(GeneratedMenuSection->FindEntry(Entry3Name));
	}

	SECTION("Generating menu with profile should hide entry 2")
	{
		FToolMenuContext MenuContext;

		UToolMenuProfileContext* ProfileContext = NewObject<UToolMenuProfileContext>();
		ProfileContext->ActiveProfiles.Add(Profile1Name);
		MenuContext.AddObject(ProfileContext);

		UToolMenu* GeneratedMenuWithProfile = UToolMenus::Get()->GenerateMenu(MenuName, MenuContext);
		REQUIRE(GeneratedMenuWithProfile);

		FToolMenuSection* GeneratedMenuSection = GeneratedMenuWithProfile->FindSection(SectionName);
		REQUIRE(GeneratedMenuSection);

		CHECK(GeneratedMenuSection->Blocks.Num() == 2);
		CHECK(GeneratedMenuSection->FindEntry(Entry1Name));
		CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry2Name));
		CHECK(GeneratedMenuSection->FindEntry(Entry3Name));
	}
}

TEST_CASE("Developer::ToolMenus::Can combine customization and multiple profiles", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	const FName MenuName("MyMenu");

	UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu(MenuName);

	REQUIRE(ToolMenu);

	// Add the section and entries
	const FName SectionName("MySection");
	const FName Entry1Name("MyEntry1");
	const FName Entry2Name("MyEntry2");
	const FName Entry3Name("MyEntry3");

	FToolMenuSection& Section = ToolMenu->AddSection(SectionName);
	Section.AddMenuEntry(Entry1Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());
	Section.AddMenuEntry(Entry2Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());
	Section.AddMenuEntry(Entry3Name, FText::GetEmpty(), FText::GetEmpty(), FSlateIcon(), FToolUIActionChoice());

	// The menu customization denies entry 1
	const FName OwnerName("ToolMenusTest");

	UToolMenus::Get()->AddRuntimeMenuCustomization(MenuName);
	FCustomizedToolMenu* CustomizedToolMenu = UToolMenus::Get()->FindRuntimeMenuCustomization(MenuName);
	REQUIRE(CustomizedToolMenu);
	CustomizedToolMenu->MenuPermissions.AddDenyListItem(OwnerName, Entry2Name);

	// Add Profile 1 which denies entry 1 and 2
	const FName Profile1Name("MyProfile1");

	UToolMenus::Get()->AddRuntimeMenuProfile(MenuName, Profile1Name);
	FToolMenuProfile* MenuProfile = UToolMenus::Get()->FindRuntimeMenuProfile(MenuName, Profile1Name);
	REQUIRE(MenuProfile);
	MenuProfile->MenuPermissions.AddDenyListItem(OwnerName, Entry1Name);
	MenuProfile->MenuPermissions.AddDenyListItem(OwnerName, Entry2Name);

	// Add Profile 2 which ONLY allows entry 1 (i.e denies ALL other entries)
	const FName Profile2Name("MyProfile2");

	UToolMenus::Get()->AddRuntimeMenuProfile(MenuName, Profile2Name);
	FToolMenuProfile* MenuProfile2 = UToolMenus::Get()->FindRuntimeMenuProfile(MenuName, Profile2Name);
	REQUIRE(MenuProfile2);
	MenuProfile2->MenuPermissions.AddAllowListItem(OwnerName, Entry1Name);


	SECTION("Generating menu with customization + profile 1 should only show entry 3")
	{
		FToolMenuContext MenuContext;
		
		UToolMenuProfileContext* ProfileContext = NewObject<UToolMenuProfileContext>();
		ProfileContext->ActiveProfiles.Add(Profile1Name);
		MenuContext.AddObject(ProfileContext);

		UToolMenu* GeneratedMenuWithProfile = UToolMenus::Get()->GenerateMenu(MenuName, MenuContext);
		REQUIRE(GeneratedMenuWithProfile);

		FToolMenuSection* GeneratedMenuSection = GeneratedMenuWithProfile->FindSection(SectionName);
		REQUIRE(GeneratedMenuSection);

		CHECK(GeneratedMenuSection->Blocks.Num() == 1);
		CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry1Name));
		CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry2Name));
		CHECK(GeneratedMenuSection->FindEntry(Entry3Name));
	}
	
	SECTION("Generating menu with customization + profile 2 should only show entry 1")
	{
		FToolMenuContext MenuContext;
		
		UToolMenuProfileContext* ProfileContext = NewObject<UToolMenuProfileContext>();
		ProfileContext->ActiveProfiles.Add(Profile2Name);
		MenuContext.AddObject(ProfileContext);

		UToolMenu* GeneratedMenuWithProfile = UToolMenus::Get()->GenerateMenu(MenuName, MenuContext);
		REQUIRE(GeneratedMenuWithProfile);

		FToolMenuSection* GeneratedMenuSection = GeneratedMenuWithProfile->FindSection(SectionName);
		REQUIRE(GeneratedMenuSection);

		CHECK(GeneratedMenuSection->Blocks.Num() == 1);
		
		CHECK(GeneratedMenuSection->FindEntry(Entry1Name));
		CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry2Name));
		CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry3Name));
	}
	
	SECTION("Generating menu with customization + profile 1 + profile 2 should only no entries")
	{
		FToolMenuContext MenuContext;

		UToolMenuProfileContext* ProfileContext = NewObject<UToolMenuProfileContext>();
		ProfileContext->ActiveProfiles.Add(Profile1Name);
		ProfileContext->ActiveProfiles.Add(Profile2Name);
		MenuContext.AddObject(ProfileContext);

		UToolMenu* GeneratedMenuWithProfile = UToolMenus::Get()->GenerateMenu(MenuName, MenuContext);
		REQUIRE(GeneratedMenuWithProfile);

		FToolMenuSection* GeneratedMenuSection = GeneratedMenuWithProfile->FindSection(SectionName);
		REQUIRE(GeneratedMenuSection);

		CHECK(GeneratedMenuSection->Blocks.Num() == 0);
		CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry1Name));
		CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry2Name));
		CHECK_FALSE(GeneratedMenuSection->FindEntry(Entry3Name));
	}
}

// Repro of UE-201151.
TEST_CASE("Developer::ToolMenus::GenerateWidget can handle simultaneous AddReferencedObjects calls in legacy dynamic sections", "[ToolMenus]")
{
	UE::ToolMenus::FToolMenuTestInstanceScoped Scope;

	UToolMenu* ToolMenu = UToolMenus::Get()->RegisterMenu("ToolMenusTest_MyMenu");

	REQUIRE(ToolMenu);

	// This simulates the crash of UE-201151 that occurred in UToolMenus::GenerateWidget(UToolMenu*) after a complex delegate
	// triggered a call to UToolMenus::AddReferencedObjects while the delegate was still executing.
	ToolMenu->AddDynamicSection(
		"MyDynamicLegacySection",
		FNewToolMenuDelegateLegacy::CreateLambda(
			[](FMenuBuilder&, UToolMenu*)
			{
				TArray<UObject*> Array;
				FReferenceFinder Finder(Array);
				UToolMenus::AddReferencedObjects(UToolMenus::Get(), Finder);
			}
		)
	);

	UToolMenus::Get()->GenerateWidget("ToolMenusTest_MyMenu", FToolMenuContext());
}
