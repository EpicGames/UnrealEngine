// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class ToolMenusTests : TestModuleRules
{
	static ToolMenusTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "ToolMenus";
			TestMetadata.TestShortName = "ToolMenus";
			TestMetadata.ReportType = "testdata";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Mac);

			string PlatformCompilationArgs;
			foreach (var Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (Platform == UnrealTargetPlatform.Android)
				{
					PlatformCompilationArgs = "-allmodules -architectures=arm64";
				}
				else
				{
					PlatformCompilationArgs = "-allmodules";
				}
				TestMetadata.PlatformCompilationExtraArgs.Add(Platform, PlatformCompilationArgs);
			}
		}
	}

	public ToolMenusTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
				"SlateCore",
				"ToolMenus"
			});

		if (Target.bBuildWithEditorOnlyData)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DesktopPlatform"
				});
		}
	}
}
