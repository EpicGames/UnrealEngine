// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class ToolMenusTestsTarget : TestTargetRules
{
	public ToolMenusTestsTarget(TargetInfo Target) : base(Target)
	{
		bTestsRequireApplicationCore = true;
		bTestsRequireCoreUObject = true;

		bBuildWithEditorOnlyData = Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop)
			&& (Target.Configuration == UnrealTargetConfiguration.Debug || Target.Configuration == UnrealTargetConfiguration.Development);
	}
}
